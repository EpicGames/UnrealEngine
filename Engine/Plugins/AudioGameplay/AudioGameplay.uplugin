{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "AudioGameplay",
	"Description": "Core plugin for audio gameplay",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AudioGameplay",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AudioGameplayTests",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}