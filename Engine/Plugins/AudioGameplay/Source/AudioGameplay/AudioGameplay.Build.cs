// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioGameplay : ModuleRules
{
	public AudioGameplay(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioExtensions",
				"Core",
				"GameplayTags",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"AudioMixerCore",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
