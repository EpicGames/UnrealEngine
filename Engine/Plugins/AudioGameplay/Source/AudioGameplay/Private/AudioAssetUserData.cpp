// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioAssetUserData.h"

#include "Sound/SoundBase.h"
#include "Sound/SoundClass.h"
#include "Templates/SubclassOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioAssetUserData)

FGameplayTagContainer UAudioAssetUserData::GetAllTags(USoundBase* InSound)
{
	FGameplayTagContainer AllTags;
	if (InSound == nullptr)
	{
		return AllTags;
	}
	
	// Check Sound directly
	if (UAudioAssetUserData* SoundData = Get(InSound))
	{
		AllTags.AppendTags(SoundData->MetadataTags);
	}
	
	// Check SoundClass
	if (UAudioAssetUserData* ClassData = Get(InSound->GetSoundClass()))
	{
		AllTags.AppendTags(ClassData->MetadataTags);
	}
	
	return AllTags;
}

bool UAudioAssetUserData::HasTag(USoundBase* InSound, FGameplayTag InTag, bool bExactMatch)
{
	FGameplayTagContainer AllTags = GetAllTags(InSound);
	if (bExactMatch)
	{
		return AllTags.HasTagExact(InTag);
	}

	return AllTags.HasTag(InTag);
}

FGameplayTagContainer UAudioAssetUserData::GetFilteredTags(USoundBase* InSound, FGameplayTag InTag)
{
	const FGameplayTagContainer AllTags = GetAllTags(InSound);
	return AllTags.Filter(FGameplayTagContainer(InTag));
}

UAudioAssetUserData* UAudioAssetUserData::Get(UObject* InObject)
{
	if (IInterface_AssetUserData* Interface = Cast<IInterface_AssetUserData>(InObject))
	{
		return Cast<UAudioAssetUserData>(Interface->GetAssetUserDataOfClass(StaticClass()));
	}
	
	return nullptr;
}
