// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioComponentGroup.h"
#include "AudioComponentGroupDebug.h"

#include "Components/AudioComponent.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Pawn.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioComponentGroup)

UAudioComponentGroup::UAudioComponentGroup(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_LastDemotable;
}

UAudioComponentGroup* UAudioComponentGroup::StaticGetOrCreateComponentGroup(AActor* Actor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAudioComponentGroup::StaticGetOrCreateComponentGroup);

	AActor* Owner = Actor;
	AActor* HighestValidOwner = nullptr;
	TInlineComponentArray<UAudioComponentGroup*> SoundGroups;

	while(Owner != nullptr)
	{
		Owner->GetComponents(SoundGroups);
		if (!SoundGroups.IsEmpty())
		{
			return SoundGroups[0];
		}

		HighestValidOwner = Owner;

		// prevents sound groups on pawns from spawning on owning playercontrollers
		if (HighestValidOwner->IsA(APawn::StaticClass()))
		{
			break;
		}
		
		Owner = Owner->GetOwner();
	}

	if(HighestValidOwner != nullptr)
	{
		UAudioComponentGroup* SoundGroup = NewObject<UAudioComponentGroup>(HighestValidOwner);
		SoundGroup->RegisterComponent();

		return SoundGroup;
	}
	
	return nullptr;
}

void UAudioComponentGroup::BeginPlay()
{
	Super::BeginPlay();
}

void UAudioComponentGroup::StopSound(USoundBase* Sound, const float FadeTime /*= 0.f*/)
{
	constexpr float TargetVolume = 0.f;

	for (TObjectPtr<UAudioComponent>& Comp : Components)
	{
		if (Comp && Comp->IsPlaying() && Comp->Sound == Sound)
		{
			Comp->FadeOut(FadeTime, TargetVolume);
		}
	}

	for (TWeakObjectPtr<UAudioComponent> Comp : ExternalComponents)
	{
		if (Comp.IsValid() && Comp->IsPlaying() && Comp->Sound == Sound)
		{
			Comp->FadeOut(FadeTime, TargetVolume);
		}
	}
}

bool UAudioComponentGroup::IsPlayingAny() const
{
	return OnStopped.IsBound();
}

void UAudioComponentGroup::BroadcastStopAll()
{
	OnStopped.Broadcast();
}

void UAudioComponentGroup::BroadcastKill()
{
	OnKilled.Broadcast();
}

void UAudioComponentGroup::BroadcastEvent(const FName EventName)
{
	if (TArray<FSoundCallback>* EventCallbacks = EventSubscriptions.Find(EventName))
	{
		for (auto EventIt = EventCallbacks->CreateIterator(); EventIt; ++EventIt)
		{
			if (EventIt->IsBound())
			{
				EventIt->Execute(EventName);
			}
			else
			{
				EventIt.RemoveCurrent();
			}
		}
	}
}

UAudioComponent* UAudioComponentGroup::GetNextAvailableComponent()
{
	UAudioComponent* ReturnComponent = nullptr;

	for (TObjectPtr<UAudioComponent>& Comp : Components)
	{
		if (Comp && !Comp->IsPlaying() && Comp->ActiveCount == 0)
		{
			ReturnComponent = ResetComponent(Comp);
			break;
		}
	}

	if (ReturnComponent == nullptr)
	{
		ReturnComponent = AddComponent();

		for (TScriptInterface<IAudioComponentGroupExtension> Extension : Extensions)
		{
			Extension->OnComponentAdded(ReturnComponent);
		}
	}

	return ReturnComponent;
}

UAudioComponent* UAudioComponentGroup::AddComponent()
{
	UAudioComponent* NewComponent = nullptr;
	if (AActor* Owner = GetOwner())
	{
		NewComponent = NewObject<UAudioComponent>(Owner);
		NewComponent->bAutoActivate = false;
		NewComponent->bAutoManageAttachment = true;

		USceneComponent* RootComponent = Owner->GetRootComponent();
		NewComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::SnapToTargetNotIncludingScale);

		if (RootComponent)
		{
			NewComponent->AutoAttachParent = RootComponent->GetAttachParent();
		}
		
		NewComponent->AutoAttachLocationRule = EAttachmentRule::KeepRelative;
		NewComponent->AutoAttachRotationRule = EAttachmentRule::KeepRelative;
		
		NewComponent->RegisterComponent();

		ApplyParams(NewComponent);

		ApplyModifiers(NewComponent, CachedModifier);

		Components.Add(NewComponent);
	}

	return NewComponent;
}

UAudioComponent* UAudioComponentGroup::ResetComponent(UAudioComponent* Component) const
{
	if (Component)
	{
		Component->SetRelativeLocationAndRotation(FVector::ZeroVector, FRotator::ZeroRotator);
		Component->ResetParameters();
		Component->bAllowSpatialization = true;

		ApplyParams(Component);

		ApplyModifiers(Component, CachedModifier);
	}

	return Component;
}

void UAudioComponentGroup::RemoveComponent(const UAudioComponent* InComponent)
{
	int32 Index = Components.IndexOfByKey(InComponent);
	if (Index != INDEX_NONE)
	{
		if (UAudioComponent* Component = Components[Index])
		{
			Component->Stop();
		}
		Components.RemoveAt(Index);
	}
}

void UAudioComponentGroup::AddExternalComponent(UAudioComponent* ComponentToAdd)
{
	if (ComponentToAdd != nullptr)
	{
		ApplyParams(ComponentToAdd);

		ApplyModifiers(ComponentToAdd, CachedModifier);

		ExternalComponents.Add(TWeakObjectPtr<UAudioComponent>(ComponentToAdd));
	}
}

void UAudioComponentGroup::RemoveExternalComponent(UAudioComponent* ComponentToRemove)
{
	if (ComponentToRemove == nullptr)
	{
		return;
	}

	ExternalComponents.Remove(ComponentToRemove);
}

void UAudioComponentGroup::EnableVirtualization()
{
	if (!bIsVirtualized)
	{
		bIsVirtualized = true;
		OnVirtualized.Broadcast();
	}
}

void UAudioComponentGroup::DisableVirtualization()
{
	if (bIsVirtualized)
	{
		// update ParamsToSet so that all parameters are updated next go-around, but merge any pending values first
		TArray<FAudioParameter> Values = ParamsToSet;
		FAudioParameter::Merge(MoveTemp(Values), PersistentParams);
		
		ParamsToSet = PersistentParams;
		
		bIsVirtualized = false;
		OnUnvirtualized.Broadcast();
	}
}

void UAudioComponentGroup::AddExtension(TScriptInterface<IAudioComponentGroupExtension> NewExtension)
{
	Extensions.AddUnique(NewExtension);
	NewExtension->OnAddedToGroup(this);
}

void UAudioComponentGroup::RemoveExtension(TScriptInterface<IAudioComponentGroupExtension> NewExtension)
{
	Extensions.Remove(NewExtension);
}

void UAudioComponentGroup::UpdateExtensions(const float DeltaTime)
{
	FAudioComponentModifier ExtensionModifier;
	for(TScriptInterface<IAudioComponentGroupExtension> Extension : Extensions)
	{
		Extension->Update(DeltaTime, this, ExtensionModifier);
	}

	ExtensionModifier.Combine(GroupModifier);

	if(bIsVirtualized)
	{
		ExtensionModifier.Volume = 0.f;
	}

	if(ExtensionModifier.IsNearlyEqual(CachedModifier) == false)
	{
		IterateComponents([this, &ExtensionModifier](UAudioComponent* Component)
		{
			ApplyModifiers(Component, ExtensionModifier);
		});
	
		CachedModifier = ExtensionModifier;
	}
}

void UAudioComponentGroup::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAudioComponentGroup::TickComponent);

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	UpdateExtensions(DeltaTime);

	UpdateComponentParameters();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FAudioComponentGroupDebug::DebugPrint(this);
#endif
}

float UAudioComponentGroup::GetFloatParamValue(const FName ParamName) const
{
	if (const FAudioParameter* FoundParam = GetParamInternal(ParamName))
	{
		return FoundParam->FloatParam;
	}

	return 0.f;
}

bool UAudioComponentGroup::GetBoolParamValue(const FName ParamName) const
{
	if (const FAudioParameter* FoundParam = GetParamInternal(ParamName))
	{
		return FoundParam->BoolParam;
	}

	return false;
}

FString UAudioComponentGroup::GetStringParamValue(const FName ParamName) const
{
	if (const FAudioParameter* FoundParam = GetParamInternal(ParamName))
	{
		return FoundParam->StringParam;
	}

	return FString();
}

void UAudioComponentGroup::SubscribeToStringParam(const FName ParamName, FStringParamCallback Delegate)
{
	if (TArray<FStringParamCallback>* StringCallbacks = StringSubscriptions.Find(ParamName))
	{
		StringCallbacks->Add(Delegate);
	}
	else
	{
		TArray<FStringParamCallback> NewSubscription = { Delegate };
		StringSubscriptions.Add(ParamName, NewSubscription);
	}
}

void UAudioComponentGroup::SubscribeToEvent(const FName EventName, FSoundCallback Delegate)
{
	if (TArray<FSoundCallback>* EventCallbacks = EventSubscriptions.Find(EventName))
	{
		EventCallbacks->AddUnique(Delegate);
	}
	else
	{
		TArray<FSoundCallback> NewSubscription = { Delegate };
		EventSubscriptions.Add(EventName, NewSubscription);
	}
}

void UAudioComponentGroup::SubscribeToBool(const FName ParamName, FBoolParamCallback Delegate)
{
	if (TArray<FBoolParamCallback>* Callbacks = BoolSubscriptions.Find(ParamName))
	{
		Callbacks->AddUnique(Delegate);
	}
	else
	{
		TArray<FBoolParamCallback> NewSubscription = { Delegate };
		BoolSubscriptions.Add(ParamName, NewSubscription);
	}
}

void UAudioComponentGroup::UnsubscribeObject(const UObject* Object)
{
	for (TPair<FName,TArray<FBoolParamCallback>>& BoolCallbacks : BoolSubscriptions)
	{
		for (TArray<FBoolParamCallback>::TIterator BoolCallbackIt = BoolCallbacks.Value.CreateIterator(); BoolCallbackIt; ++BoolCallbackIt)
		{
			if (!BoolCallbackIt->IsBound() || BoolCallbackIt->GetUObject() == Object)
			{
				BoolCallbackIt.RemoveCurrent();
			}
		}
	}

	for (TPair<FName,TArray<FStringParamCallback>>& StringCallbacks : StringSubscriptions)
	{
		for (TArray<FStringParamCallback>::TIterator StringCallbackIt = StringCallbacks.Value.CreateIterator(); StringCallbackIt; ++StringCallbackIt)
		{
			if (!StringCallbackIt->IsBound() || StringCallbackIt->GetUObject() == Object)
			{
				StringCallbackIt.RemoveCurrent();
			}
		}
	}

	for (TPair<FName,TArray<FSoundCallback>>& Callbacks : EventSubscriptions)
	{
		for (TArray<FSoundCallback>::TIterator CallbackIt = Callbacks.Value.CreateIterator(); CallbackIt; ++CallbackIt)
		{
			if (!CallbackIt->IsBound() || CallbackIt->GetUObject() == Object)
			{
				CallbackIt.RemoveCurrent();
			}
		}
	}
}

void UAudioComponentGroup::ResetParameters()
{
	PersistentParams.Reset();
}

void UAudioComponentGroup::SetTriggerParameter(FName InName)
{
	ExecuteEventSubscriptions(InName);

	IterateComponents([InName](UAudioComponent* Component)
	{
		if (Component->IsPlaying())
		{
			Component->SetTriggerParameter(InName);
		}
	});
}

void UAudioComponentGroup::SetBoolParameter(FName InName, bool InBool)
{
	SetParameters({ FAudioParameter(InName, InBool) });
}

void UAudioComponentGroup::SetBoolArrayParameter(FName InName, const TArray<bool>& InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void UAudioComponentGroup::SetIntParameter(FName InName, int32 InInt)
{
	SetParameters({ FAudioParameter(InName, InInt) });
}

void UAudioComponentGroup::SetIntArrayParameter(FName InName, const TArray<int32>& InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void UAudioComponentGroup::SetFloatParameter(FName InName, float InFloat)
{
	SetParameters({ FAudioParameter(InName, InFloat) });
}

void UAudioComponentGroup::SetFloatArrayParameter(FName InName, const TArray<float>& InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void UAudioComponentGroup::SetStringParameter(FName InName, const FString& InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void UAudioComponentGroup::SetStringArrayParameter(FName InName, const TArray<FString>& InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void UAudioComponentGroup::SetObjectParameter(FName InName, UObject* InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void UAudioComponentGroup::SetObjectArrayParameter(FName InName, const TArray<UObject*>& InValue)
{
	SetParameters({ FAudioParameter(InName, InValue) });
}

void UAudioComponentGroup::SetParameters_Blueprint(const TArray<FAudioParameter>& InParameters)
{
	TArray<FAudioParameter> Values = InParameters;
	SetParameters(MoveTemp(Values));
}

void UAudioComponentGroup::SetParameter(FAudioParameter&& InValue)
{
	SetParameters({ MoveTemp(InValue) });
}

void UAudioComponentGroup::SetParameters(TArray<FAudioParameter>&& InValues)
{
	InValues.RemoveAll([](FAudioParameter& Param) { return Param.ParamName.IsNone(); });

	for (FAudioParameter& Parameter : InValues)
	{
		switch (Parameter.ParamType)
		{
		case EAudioParameterType::Boolean:
			if (GetBoolParamValue(Parameter.ParamName) != Parameter.BoolParam)
			{
				ExecuteBoolParamSubscriptions(Parameter);
			}
			break;
		case EAudioParameterType::Trigger:
			ExecuteEventSubscriptions(Parameter.ParamName);
			break;
		case EAudioParameterType::String:
			if (GetStringParamValue(Parameter.ParamName) != Parameter.StringParam)
			{
				ExecuteStringParamSubscriptions(Parameter);
			}
			break;
		default:
			break;
		}
	}
	
	FAudioParameter::Merge(MoveTemp(InValues), ParamsToSet);
}

void UAudioComponentGroup::ApplyParams(UAudioComponent* Component) const
{
	if (Component)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAudioComponentGroup::ApplyParams);

		// if the component is playing already, InstanceParameters will be ignored
		if (Component->IsPlaying())
		{
			TArray<FAudioParameter> Params = PersistentParams;
			Component->SetParameters(MoveTemp(Params));
		}
		else
		{
			// if the component hasn't started playing yet, all of its instance parameters will be passed along at once when it starts
			TArray<FAudioParameter>& InstanceParameters = Component->GetInstanceParameters();
			InstanceParameters.Append(PersistentParams);
		}
	}
}

void UAudioComponentGroup::ApplyModifiers(UAudioComponent* Component, const FAudioComponentModifier& Modifier) const
{
	if (Component)
	{
		Component->SetVolumeMultiplier(Modifier.Volume);
		Component->SetPitchMultiplier(Modifier.Pitch);
		Component->SetLowPassFilterFrequency(Modifier.LowPassFrequency);
	}
}

void UAudioComponentGroup::IterateComponents(const TFunction<void(UAudioComponent*)> OnIterate)
{
	Components.RemoveAll([](const UAudioComponent* Component) { return !Component; });
	for (UAudioComponent* Component : Components)
	{
		if (Component->IsActive())
		{
			OnIterate(Component);
		}
	}

	ExternalComponents.RemoveAll([](const TWeakObjectPtr<UAudioComponent> WeakComponent) { return !WeakComponent.IsValid(); });
	for (TWeakObjectPtr<UAudioComponent>& WeakComponent : ExternalComponents)
	{
		OnIterate(WeakComponent.Get());
	}
}

void UAudioComponentGroup::UpdateComponentParameters()
{
	if(ParamsToSet.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UAudioComponentGroup::UpdateComponentParameters);

	if (bIsVirtualized == false)
	{
		IterateComponents([this](UAudioComponent* Component)
    	{
    		// todo: this could be a LOT of tiny allocs... can anything be done about that?
    		TArray<FAudioParameter> Values = ParamsToSet;
    		Component->SetParameters(MoveTemp(Values));
    	});
	}
	
	FAudioParameter::Merge(MoveTemp(ParamsToSet), PersistentParams);
	ParamsToSet.Reset();
}

float UAudioComponentGroup::GetComponentVolume() const
{
	if (bIsVirtualized)
	{
		return 0.f;
	}

	return CachedModifier.Volume;
}

void UAudioComponentGroup::ExecuteStringParamSubscriptions(const FAudioParameter& StringParam)
{
	if (StringParam.ParamType != EAudioParameterType::String)
	{
		return;
	}
	
	if (TArray<FStringParamCallback>* StringCallbacks = StringSubscriptions.Find(StringParam.ParamName))
	{
		for (TArray<FStringParamCallback>::TIterator StringCallbackIt = StringCallbacks->CreateIterator(); StringCallbackIt; ++StringCallbackIt)
		{
			if (StringCallbackIt->IsBound())
			{
				StringCallbackIt->Execute(StringParam.StringParam);
			}
			else
			{
				StringCallbackIt.RemoveCurrent();
			}
		}
	}
}

void UAudioComponentGroup::ExecuteBoolParamSubscriptions(const FAudioParameter& BoolParam)
{
	if (BoolParam.ParamType != EAudioParameterType::Boolean)
	{
		return;
	}
	
	if (TArray<FBoolParamCallback>* BoolCallbacks = BoolSubscriptions.Find(BoolParam.ParamName))
	{
		for (TArray<FBoolParamCallback>::TIterator BoolCallbackIt = BoolCallbacks->CreateIterator(); BoolCallbackIt; ++BoolCallbackIt)
		{
			if (BoolCallbackIt->IsBound())
			{
				BoolCallbackIt->Execute(BoolParam.BoolParam);
			}
			else
			{
				BoolCallbackIt.RemoveCurrent();
			}
		}
	}
}

void UAudioComponentGroup::ExecuteEventSubscriptions(const FName EventName)
{
	if (EventName.IsNone())
	{
		return;
	}
	
	if (TArray<FSoundCallback>* Callbacks = EventSubscriptions.Find(EventName))
	{
		for (TArray<FSoundCallback>::TIterator CallbackIt = Callbacks->CreateIterator(); CallbackIt; ++CallbackIt)
		{
			if (CallbackIt->IsBound())
			{
				CallbackIt->Execute(EventName);
			}
			else
			{
				CallbackIt.RemoveCurrent();
			}
		}
	}
}

const FAudioParameter* UAudioComponentGroup::GetParamInternal(const FName ParamName) const
{
	return PersistentParams.FindByPredicate([ParamName](const FAudioParameter& Param) { return Param.ParamName == ParamName; });
}
