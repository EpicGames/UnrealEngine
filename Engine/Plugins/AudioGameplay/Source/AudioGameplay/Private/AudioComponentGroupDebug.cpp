// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioComponentGroupDebug.h"
#include "AudioComponentGroup.h"
#include "Components/AudioComponent.h"
#include "DrawDebugHelpers.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "Sound/SoundBase.h"

namespace AudioComponentGroupCVars 
{
	int32 PrintSoundParams = 0;
	static FAutoConsoleVariableRef CVarPrintSoundParams(
		TEXT("au.AudioComponentGroup.debug.PrintSoundParams"),
		PrintSoundParams,
		TEXT("Set to 1 to print sound parameters for each actor.\n0: Disable (default), 1: Enable"),
		ECVF_Cheat );

	int32 PrintActiveSounds = 0;
	static FAutoConsoleVariableRef CVarPrintActiveSounds(
		TEXT("au.AudioComponentGroup.debug.PrintActiveSounds"),
		PrintActiveSounds,
		TEXT("Display the active sounds on each actor.\n0: Disable (default), 1: Enable"),
		ECVF_Cheat );

	FString PrintSoundParamsActorFilter = FString(TEXT("All"));
	static FAutoConsoleVariableRef CVarPrintSoundParamsActorFilter(
		TEXT("au.AudioComponentGroup.debug.ActorFilter"),
		PrintSoundParamsActorFilter,
		TEXT("Only display params on actors that contain this string. Disable filtering with All"),
		ECVF_Cheat );

	FString PrintSoundParamsParamFilter = FString(TEXT("All"));
	static FAutoConsoleVariableRef CVarPrintSoundParamsParamFilter(
		TEXT("au.AudioComponentGroup.debug.ParamFilter"),
		PrintSoundParamsParamFilter,
		TEXT("Only display params with names that contain this string. Disable filtering with All"),
		ECVF_Cheat );

	float ZOffset = 150.0f;
	static FAutoConsoleVariableRef CVarPrintZOffset(
		TEXT("au.AudioComponentGroup.debug.ActorZOffset"),
		ZOffset,
		TEXT("How High above an actor to print debug info. Default: 150"),
		ECVF_Cheat );
}

void FAudioComponentGroupDebug::DebugPrint(const UAudioComponentGroup* ComponentGroup)
{
	using namespace AudioComponentGroupCVars ;
	
	if(ComponentGroup == nullptr)
	{
		return;
	}
	
	if(LIKELY(PrintSoundParams <= 0 && PrintActiveSounds <= 0))
	{
		return;
	}

	const AActor* ComponentGroupOwner = ComponentGroup->GetOwner();
	check(ComponentGroupOwner);
	
	if (PrintSoundParamsActorFilter.Equals(TEXT("All"), ESearchCase::IgnoreCase) == false 
		&& ComponentGroupOwner->GetName().Contains(PrintSoundParamsActorFilter) == false)
	{
		return;
	}

	FString OutString = ComponentGroup->GetOwner()->GetName();
	OutString += LINE_TERMINATOR;
	
	if (UNLIKELY(PrintSoundParams >= 1))
	{
		OutString += DebugPrintParams(ComponentGroup);
	}

	if(UNLIKELY(PrintActiveSounds >= 1))
	{
		OutString += DebugPrintActiveSounds(ComponentGroup);
	}

	PrintDebugString(OutString, ComponentGroup->GetOwner(), GetDebugStringColor(ComponentGroup->bIsVirtualized));
}

FString FAudioComponentGroupDebug::DebugPrintParams(const UAudioComponentGroup* ComponentGroup)
{
	using namespace AudioComponentGroupCVars ;

	FString OutString;

	const bool bPrintAllParams = PrintSoundParamsParamFilter.Equals(TEXT("All"), ESearchCase::IgnoreCase);
	
	for (const FAudioParameter& Param : ComponentGroup->PersistentParams)
	{
		if (bPrintAllParams || Param.ParamName.ToString().Contains(PrintSoundParamsParamFilter))
		{
			OutString += GetDebugParamString(Param);
		}
	}

	OutString += LINE_TERMINATOR;

	return OutString;
}

FColor FAudioComponentGroupDebug::GetDebugStringColor(const bool bVirtualized)
{
	if(bVirtualized)
	{
		return FColor::Yellow;
	}

	return FColor::Green;
}

FString FAudioComponentGroupDebug::GetDebugParamString(const FAudioParameter& Param)
{
	switch (Param.ParamType)
	{
	case EAudioParameterType::Boolean:
		return FString::Printf(TEXT("%s : %s \n"), *Param.ParamName.ToString(), Param.BoolParam ? TEXT("true") : TEXT("false"));
	case EAudioParameterType::Float:
		return FString::Printf(TEXT("%s : %f \n"), *Param.ParamName.ToString(), Param.FloatParam);
	case EAudioParameterType::Integer:
		return FString::Printf(TEXT("%s : %d \n"), *Param.ParamName.ToString(), Param.IntParam);
	case EAudioParameterType::String:
		return FString::Printf(TEXT("%s : %s \n"), *Param.ParamName.ToString(), *Param.StringParam);
	default:
		return FString::Printf(TEXT("%s : Unsupported Type \n"), *Param.ParamName.ToString());
	}
}

FString FAudioComponentGroupDebug::DebugPrintActiveSounds(const UAudioComponentGroup* ComponentGroup)
{
	FString OutString;

	for(const UAudioComponent* Component : ComponentGroup->Components)
	{
		if(Component && Component->Sound && Component->IsActive())
		{
			OutString += FString::Printf(TEXT("%s \n"), *(Component->Sound->GetName()));
		}
	}

	for(const TWeakObjectPtr<UAudioComponent>& Component : ComponentGroup->ExternalComponents)
	{
		if(Component.IsValid() && Component->Sound && Component->IsActive())
		{
			OutString += FString::Printf(TEXT("%s \n"), *(Component->Sound->GetName()));
		}
	}

	OutString += LINE_TERMINATOR;

	return OutString;
}

void FAudioComponentGroupDebug::PrintDebugString(const FString& InString, AActor* Owner, const FColor DrawColor)
{
	if(Owner == nullptr)
	{
		return;
	}
	
	DrawDebugString(Owner->GetWorld(), FVector(0.0f, 0.0f, AudioComponentGroupCVars ::ZOffset), *InString, Owner, DrawColor, 0.0f, true);
}
