// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayComponent.h"
#include "AudioGameplayLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayComponent)

UAudioGameplayComponent::UAudioGameplayComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	PrimaryComponentTick.SetTickFunctionEnable(false);
}

void UAudioGameplayComponent::Activate(bool bReset)
{
	const bool bWasActive = IsActive();
	Super::Activate(bReset);

	if (!bWasActive && IsActive() && GetNetMode() != NM_DedicatedServer)
	{
		Enable();
	}
}

void UAudioGameplayComponent::Deactivate()
{
	const bool bWasActive = IsActive();
	Super::Deactivate();

	if (bWasActive && !IsActive() && GetNetMode() != NM_DedicatedServer)
	{
		Disable();
	}
}

void UAudioGameplayComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Deactivate();
	Super::EndPlay(EndPlayReason);
}

bool UAudioGameplayComponent::HasPayloadType(PayloadFlags InType) const
{
	return (PayloadType & InType) != PayloadFlags::AGCP_None;
}

void UAudioGameplayComponent::Enable()
{
	UE_LOGF(AudioGameplayLog, VeryVerbose, "AudioGameplayComponent Enabled (%ls).", *GetFullName());
}

void UAudioGameplayComponent::Disable()
{
	UE_LOGF(AudioGameplayLog, VeryVerbose, "AudioGameplayComponent Disabled (%ls).", *GetFullName());
}

