// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayModule.h"
#include "AudioGameplayLogs.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"

#define LOCTEXT_NAMESPACE "FAudioGameplay"

DEFINE_LOG_CATEGORY(AudioGameplayLog);

// Defines the "AudioGameplay" category in the CSV profiler.
// This should only be defined here. Modules who wish to use this category should contain the line
// 		CSV_DECLARE_CATEGORY_MODULE_EXTERN(AUDIOGAMEPLAY_API, AudioGameplay);
//

#if UE_BUILD_SHIPPING
CSV_DEFINE_CATEGORY_MODULE(AUDIOGAMEPLAY_API, AudioGameplay, false);
#else
CSV_DEFINE_CATEGORY_MODULE(AUDIOGAMEPLAY_API, AudioGameplay, true);
#endif

void FAudioGameplayModule::StartupModule()
{
	UE_LOGF(AudioGameplayLog, Log, "%ls", ANSI_TO_TCHAR(__FUNCTION__));
}

void FAudioGameplayModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAudioGameplayModule, AudioGameplay)
