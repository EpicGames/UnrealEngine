// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayRequirements.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayRequirements)

UAudioRequirementPreset::UAudioRequirementPreset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UAudioRequirementPreset::Matches(const FGameplayTagContainer& Tags) const
{
	return Query.IsEmpty() || Query.Matches(Tags);
}

bool FAudioGameplayRequirements::Matches(const FGameplayTagContainer& Tags) const
{
	if (Preset && !Preset->Matches(Tags))
	{
		return false;
	}

	if (!Custom.IsEmpty() && !Custom.Matches(Tags))
	{
		return false;
	}

	return true;
}
