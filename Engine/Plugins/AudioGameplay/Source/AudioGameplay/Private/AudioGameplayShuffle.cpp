// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayShuffle.h"

namespace UE::Audio
{
	void FShuffleUtil::Initialize(int32 ArraySize)
	{
		ShuffleArray.SetNum(ArraySize);

		for (int32 i = 0; i < ShuffleArray.Num(); i++)
		{
			ShuffleArray[i] = i;
		}

		Shuffle();
	}

	uint8 FShuffleUtil::GetNextIndex()
	{
		const int32 ShuffleArraySize = ShuffleArray.Num();
		if (ShuffleArraySize == 0)
		{
			return INDEX_NONE;
		}

		if (ShuffleMarker > ShuffleArraySize)
		{
			Shuffle();
		}

		// Post decrement will setup an underflow to cause a shuffle on next access
		return ShuffleArray[ShuffleMarker--];
	}

	void FShuffleUtil::Shuffle()
	{
		const int32 LastIndex = ShuffleArray.Num() - 1;
		for (int32 i = 0; i <= LastIndex; ++i)
		{
			const int32 SwapIndex = FMath::RandRange(i, LastIndex);
			if (i != SwapIndex)
			{
				ShuffleArray.Swap(i, SwapIndex);
			}
		}

		ShuffleMarker = LastIndex;
	}
} // namespace UE::Audio
