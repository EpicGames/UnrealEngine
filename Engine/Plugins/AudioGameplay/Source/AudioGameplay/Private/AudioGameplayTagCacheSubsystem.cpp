// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayTagCacheSubsystem.h"

#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayTagCacheSubsystem)

namespace AudioGameplayTagCacheConsoleVariables
{
	int32 bEnabled = 1;
	FAutoConsoleVariableRef CVarEnabled(
		TEXT("au.GameplayTagCache.Enabled"),
		bEnabled,
		TEXT("Cache any string concatenations used to build Gameplay Tags at runtime.\n0: Disable, 1: Enable (default)"),
		ECVF_Default);

} // namespace AudioGameplayTagCacheConsoleVariables

bool UAudioGameplayTagCacheSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (AudioGameplayTagCacheConsoleVariables::bEnabled == 0)
	{
		return false;
	}

	return Super::ShouldCreateSubsystem(Outer);
}

void UAudioGameplayTagCacheSubsystem::Deinitialize()
{
	GameplayTagCache.Empty();

	Super::Deinitialize();
}

UAudioGameplayTagCacheSubsystem* UAudioGameplayTagCacheSubsystem::Get(const UWorld* WorldContext)
{
	if (WorldContext)
	{
		return WorldContext->GetSubsystem<UAudioGameplayTagCacheSubsystem>();
	}

	return nullptr;
}
