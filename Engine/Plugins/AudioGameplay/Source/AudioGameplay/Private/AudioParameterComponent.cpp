// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioParameterComponent.h"
#include "AudioDevice.h"
#include "ActiveSound.h"
#include "IAudioParameterTransmitter.h"
#include "Kismet/KismetStringLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioParameterComponent)

namespace AudioParameterComponentConsoleVariables
{
	int32 bSetParamOnlyOnValueChange = 1;
	FAutoConsoleVariableRef CVarSetParamOnlyOnValueChange(
		TEXT("au.AudioParameterComponent.SetParamOnlyOnValueChange"),
		bSetParamOnlyOnValueChange,
		TEXT("Only sets parameters when the underlying value has changed.\n0: Disable, 1: Enable (default)"),
		ECVF_Default);
}

namespace AudioParameterComponentUtils
{
	// This one is a bit weird, but we combine params by merging them - and we currently 
	// always take the parameter type of the thing being merged in.  This means for something to be 
	// Equivalent (for a no-op type optimization) it has to have the same ParamType
	bool TestEquivalence(const FAudioParameter& Lhs, const FAudioParameter& Rhs)
	{
		if (Lhs.ParamType != Rhs.ParamType)
		{
			return false;
		}

		switch (Rhs.ParamType)
		{
		case EAudioParameterType::Boolean:
		{
			return Lhs.BoolParam == Rhs.BoolParam;
		}

		case EAudioParameterType::BooleanArray:
		{
			return Lhs.ArrayBoolParam == Rhs.ArrayBoolParam;
		}

		case EAudioParameterType::Float:
		{
			return FMath::IsNearlyEqual(Lhs.FloatParam, Rhs.FloatParam);
		}

		case EAudioParameterType::FloatArray:
		{
			return Lhs.ArrayFloatParam == Rhs.ArrayFloatParam;
		}

		case EAudioParameterType::Integer: 
		case EAudioParameterType::NoneArray:
		{
			return Lhs.IntParam == Rhs.IntParam;
		}

		case EAudioParameterType::IntegerArray:
		{
			return Lhs.ArrayIntParam == Rhs.ArrayIntParam;
		}

		case EAudioParameterType::None:
		{
			// Moved string comp to end hoping for lazy eval
			return FMath::IsNearlyEqual(Lhs.FloatParam, Rhs.FloatParam) && Lhs.BoolParam == Rhs.BoolParam && Lhs.IntParam == Rhs.IntParam &&
				Lhs.ObjectParam == Rhs.ObjectParam && Lhs.ArrayFloatParam == Rhs.ArrayFloatParam && Lhs.ArrayBoolParam == Rhs.ArrayBoolParam &&
				Lhs.ArrayIntParam == Rhs.ArrayIntParam && Lhs.ArrayObjectParam == Rhs.ArrayObjectParam &&
				Lhs.StringParam == Rhs.StringParam && Lhs.ArrayStringParam == Rhs.ArrayStringParam;
		}

		case EAudioParameterType::Object:
		{
			return Lhs.ObjectParam == Rhs.ObjectParam;
		}

		case EAudioParameterType::ObjectArray:
		{
			return Lhs.ArrayObjectParam == Rhs.ArrayObjectParam;
		}

		case EAudioParameterType::String:
		{
			return Lhs.StringParam == Rhs.StringParam;
		}

		case EAudioParameterType::StringArray:
		{
			return Lhs.ArrayStringParam == Rhs.ArrayStringParam;
		}

		default:
		break;

		}

		return false;
	}
}

DEFINE_LOG_CATEGORY(LogAudioParameterComponent);

void UAudioParameterComponent::GetActorSoundParams_Implementation(TArray<FAudioParameter>& Params) const
{
	Params.Append(GetParameters());
}

void UAudioParameterComponent::ResetParameters()
{
	int32 CurrSize = Parameters.Num();
	Parameters.Reset(CurrSize);
}

void UAudioParameterComponent::SetTriggerParameter(FName InName)
{
	// todo@sweedin: use "GetAllAudioComponent" and execute this trigger on them
	UE_LOGF(LogAudio, Warning, "Trigger parameter not supported for UAudioParameterComponent");
}

void UAudioParameterComponent::SetBoolParameter(FName InName, bool InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetBoolArrayParameter(FName InName, const TArray<bool>& InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetIntParameter(FName InName, int32 InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetIntArrayParameter(FName InName, const TArray<int32>& InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetFloatParameter(FName InName, float InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetFloatArrayParameter(FName InName, const TArray<float>& InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetStringParameter(FName InName, const FString& InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetStringArrayParameter(FName InName, const TArray<FString>& InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetObjectParameter(FName InName, UObject* InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetObjectArrayParameter(FName InName, const TArray<UObject*>& InValue)
{
	SetParameterInternal(FAudioParameter(InName, InValue));
}

void UAudioParameterComponent::SetParameters_Blueprint(const TArray<FAudioParameter>& InParameters)
{
	TArray<FAudioParameter> Values = InParameters;
	SetParameters(MoveTemp(Values));
}

void UAudioParameterComponent::SetParameter(FAudioParameter&& InValue)
{
	SetParameterInternal(MoveTemp(InValue));
}

void UAudioParameterComponent::SetParameters(TArray<FAudioParameter>&& InValues)
{
	for (const FAudioParameter& Value : InValues)
	{
		if (FAudioParameter* CurrentParam = FAudioParameter::FindOrAddParam(Parameters, Value.ParamName))
		{
			constexpr bool bInTakeName = false;
			CurrentParam->Merge(Value, bInTakeName);
		}
	}

	// Forward to any AudioComponents currently playing on this actor (if any)
	TArray<TObjectPtr<UAudioComponent>> Components;
	GetAllAudioComponents(MutableView(Components));

	for (auto& Component : Components)
	{
		if (Component && Component->IsPlaying())
		{
			TArray<FAudioParameter> ValuesTemp = InValues;
			Component->SetParameters(MoveTemp(ValuesTemp));
		}
	}
}

void UAudioParameterComponent::SetParameterInternal(FAudioParameter&& InParam)
{
	if (InParam.ParamName.IsNone())
	{
		return;
	}

	const int32 StartingParamCount = Parameters.Num();
	if (FAudioParameter* CurrentParam = FAudioParameter::FindOrAddParam(Parameters, InParam.ParamName))
	{
		if (AudioParameterComponentConsoleVariables::bSetParamOnlyOnValueChange != 0)
		{
			if (StartingParamCount == Parameters.Num())
			{
				// Early out if we did not add a parameter and this one matches.
				// We always take the type of the param (see merge below), so the utility function below
				// is only going to test based on changes to the right hand side 
				if  (AudioParameterComponentUtils::TestEquivalence(*CurrentParam, InParam))
				{
					return;
				}
			}
		}

		constexpr bool bInTakeName = false;
		CurrentParam->Merge(InParam, bInTakeName);
	}

	// Optional logging
	LogParameter(InParam);

	// Optional broadcast (for editor-usage only)
#if WITH_EDITORONLY_DATA
	if (OnParameterChanged.IsBound())
	{
		OnParameterChanged.Broadcast(InParam);
	}
#endif // WITH_EDITORONLY_DATA

	// Forward to any AudioComponents currently playing on this actor (if any)
	TArray<UAudioComponent*> Components;
	GetAllAudioComponents(Components);

	TArray<FAudioParameter> ParametersCopy;
	for (auto& Component : Components)
	{
		if (Component == nullptr)
		{
			continue;
		}

		if (Component->IsPlaying() && !Component->GetDisableParameterUpdatesWhilePlaying())
		{
			if (FAudioDevice* AudioDevice = Component->GetAudioDevice())
			{
				ParametersCopy = Parameters;
				if (USoundBase* Sound = Component->GetSound())
				{
					static const FName ProxyFeatureName("AudioParameterComponent");
					Sound->InitParameters(ParametersCopy, ProxyFeatureName);
				}

				// Prior call to InitParameters can prune parameters if they are
				// invalid, so check here to avoid unnecessary pass of empty array.
				if (!ParametersCopy.IsEmpty())
				{
					DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.SoundParameterControllerInterface.SetParameters"), STAT_AudioSetParameters, STATGROUP_AudioThreadCommands);

					AudioDevice->SendCommandToActiveSounds(Component->GetInstanceOwnerID(), [AudioDevice, Params = MoveTemp(ParametersCopy)](FActiveSound& ActiveSound)
					{
						if (Audio::IParameterTransmitter* Transmitter = ActiveSound.GetTransmitter())
						{
							TArray<FAudioParameter> TempParams = Params;
							Transmitter->SetParameters(MoveTemp(TempParams));
						}
					}, GET_STATID(STAT_AudioSetParameters));
				}
			}
		}
	}
}

void UAudioParameterComponent::GetAllAudioComponents(TArray<UAudioComponent*>& Components) const
{
	AActor* RootActor = GetOwner();
	if (USceneComponent* RootComp = RootActor->GetRootComponent())
	{
		// Walk up to the top-most attach actor in the hierarchy (will just be the RootActor if no attachment)
		RootActor = RootComp->GetAttachmentRootActor();
	}

	// Helper to collect components from an actor
	auto CollectFromActor = [&Components](const AActor* InActor)
	{
		if (InActor)
		{
			TArray<UAudioComponent*> TempComponents;
			constexpr bool bIncludeChildActors = false;
			InActor->GetComponents<UAudioComponent>(TempComponents, bIncludeChildActors);
			Components.Append(MoveTemp(TempComponents));
		}
	};

	if (RootActor)
	{
		// Grab any AudioComponents from our owner directly
		CollectFromActor(RootActor);

		// Recursively grab AudioComponents from our Attached Actor's
		TArray<AActor*> AttachedActors;
		constexpr bool bResetArray = false;
		constexpr bool bRecursivelyIncludeAttachedActors = true;
		RootActor->GetAttachedActors(AttachedActors, bResetArray, bRecursivelyIncludeAttachedActors);

		for (AActor* Actor : AttachedActors)
		{
			CollectFromActor(Actor);
		}
	}
}

void UAudioParameterComponent::LogParameter(FAudioParameter& InParam)
{
	if (!UE_LOG_ACTIVE(LogAudioParameterComponent, VeryVerbose))
	{
		return;
	}

	const AActor* Owner = GetOwner();
	if (Owner == nullptr)
	{
		return;
	}

	FString ParamTypeString;
	const UEnum* ParamTypeEnum = StaticEnum<EAudioParameterType>();
	if (ParamTypeEnum != nullptr)
	{
		ParamTypeString = ParamTypeEnum->GetNameStringByIndex(static_cast<int32>(InParam.ParamType));
	}

	FString ParamValueString;
	switch (InParam.ParamType)
	{
		case EAudioParameterType::Boolean: 
			ParamValueString = UKismetStringLibrary::Conv_BoolToString(InParam.BoolParam);
			break;

		case EAudioParameterType::Integer:
			ParamValueString = UKismetStringLibrary::Conv_IntToString(InParam.IntParam);
			break;

		case EAudioParameterType::Float:
			ParamValueString = FString::SanitizeFloat(InParam.FloatParam);
			break;
	}

	UE_LOGF(LogAudioParameterComponent, VeryVerbose, "%ls: Set %ls %ls to %ls", *Owner->GetName(), *ParamTypeString, *InParam.ParamName.ToString(), *ParamValueString)
}
