// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilteredGameplayTagContainer.h"

bool FFilteredGameplayTagContainer::AddTagFiltered(const FilteredGPTContainer::FGPTagQueryPair& InPair)
{
	if (!AddTagToContainers(InPair))
	{
		return false;
	}

	FilterTags();
	return true;
}

bool FFilteredGameplayTagContainer::AddTagFiltered(const FGameplayTag& InTag, const FGameplayTagQuery& InQuery /*= FGameplayTagQuery()*/)
{
	return AddTagFiltered(FilteredGPTContainer::FGPTagQueryPair(InTag, InQuery));
}


bool FFilteredGameplayTagContainer::RemoveTagFiltered(const FGameplayTag& InTag)
{
	if (!RemoveTagFromContainers(InTag))
	{
		return false;
	}

	FilterTags();
	return true;
}

bool FFilteredGameplayTagContainer::AddTagToContainers(const FilteredGPTContainer::FGPTagQueryPair& InPair)
{
	if (!TagContainer.HasTagExact(InPair.Key) && (InPair.Value.IsEmpty() || TagContainer.MatchesQuery(InPair.Value)))
	{
		GPTagQueryPairContainer.Add(InPair);
		TagContainer.AddTagFast(InPair.Key);
		OnGameplayTagAdded.Broadcast(InPair.Key);
		return true;
	}

	return false;
}

bool FFilteredGameplayTagContainer::RemoveTagFromContainers(const FGameplayTag& Tag)
{
	for (int32 i = GPTagQueryPairContainer.Num() - 1; i >= 0; --i)
	{
		if (GPTagQueryPairContainer[i].Key == Tag)
		{
			GPTagQueryPairContainer.RemoveAt(i);
			TagContainer.RemoveTag(Tag);
			OnGameplayTagRemoved.Broadcast(Tag);
			return true;

		}
	}

	return false;	
}

void FFilteredGameplayTagContainer::FilterTags()
{
	for (int32 i = GPTagQueryPairContainer.Num() - 1; i >= 0; --i)
	{
		const FilteredGPTContainer::FGPTagQueryPair& TagQueryPair = GPTagQueryPairContainer[i];

		if (TagQueryPair.Value.IsEmpty())
		{
			continue;
		}

		if (!TagContainer.MatchesQuery(TagQueryPair.Value))
		{
			FGameplayTag TagToRemove = TagQueryPair.Key;
			GPTagQueryPairContainer.RemoveAt(i);
			TagContainer.RemoveTag(TagToRemove);
			OnGameplayTagRemoved.Broadcast(TagToRemove);
		}
	}

	CheckContainersConsistency();
}

void FFilteredGameplayTagContainer::CheckContainersConsistency()
{
	ensureMsgf(GPTagQueryPairContainer.Num() == TagContainer.Num(), TEXT("Expected Event and Tag Container to have the same lenght. GPTagQueryPairContainer: %d, TagContainer: %d"), GPTagQueryPairContainer.Num(), TagContainer.Num());
}
