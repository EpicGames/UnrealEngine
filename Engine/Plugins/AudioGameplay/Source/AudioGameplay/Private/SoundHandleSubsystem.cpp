// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundHandleSubsystem.h"

#include "ActiveSound.h"
#include "AudioDevice.h"
#include "Audio/AudioTraceUtil.h"
#include "Audio/ISoundHandleOwner.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoundHandleSubsystem)

bool USoundHandleSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return Super::ShouldCreateSubsystem(Outer) && !IsRunningDedicatedServer();
}

void USoundHandleSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
}

void USoundHandleSubsystem::Deinitialize()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
	ActiveHandles.Reset();
	Super::Deinitialize();
}

void USoundHandleSubsystem::NotifyActiveSoundDeleting(const FActiveSound& ActiveSound)
{
	NotifySoundDeleting(ActiveSound);
}

void USoundHandleSubsystem::NotifyVirtualizedSoundDeleting(const FActiveSound& ActiveSound)
{
	NotifySoundDeleting(ActiveSound);
}

void USoundHandleSubsystem::NotifySoundDeleting(const FActiveSound& ActiveSound)
{
	check(IsInAudioThread());
	if (ActiveHandles.Num() > 0)
	{
		const Audio::FSoundHandleID HandleID = ActiveSound.GetInstanceID();
		ActiveHandles.Remove(HandleID);
		if(ISoundHandleOwner* Owner = Owners.FindAndRemoveChecked(HandleID))
		{			
			Owner->OnSoundHandleRemoved();
		}
	}
}

Audio::FSoundHandleID USoundHandleSubsystem::CreateSoundHandle(USoundBase* Sound, ISoundHandleOwner* Owner)
{
	if(Sound && Owner)
	{
		FSoundHandle NewHandle;
		NewHandle.ActiveSound.bLocationDefined = false;
		Audio::FSoundHandleID HandleID = NewHandle.ActiveSound.GetInstanceID();
		NewHandle.ActiveSound.SetSound(Sound);
		ActiveHandles.Emplace(HandleID, NewHandle);
		Owners.Emplace(HandleID, Owner);
		return HandleID;
	}
	return INDEX_NONE;
}

void USoundHandleSubsystem::SetTransform(const Audio::FSoundHandleID ID, const FTransform& Transform)
{
	// TODO: We should add this as an incoming change to handle on the audio thread in the update
	if(FSoundHandle* Handle = ActiveHandles.Find(ID))
	{
		Handle->ActiveSound.bLocationDefined = true;
		Handle->ActiveSound.Transform = Transform;
	}	
}

Audio::EResult USoundHandleSubsystem::Play(const Audio::FSoundHandleID ID)
{
	if(FSoundHandle* Handle = ActiveHandles.Find(ID))
	{
		FAudioDevice* AudioDevice = GetAudioDeviceHandle().GetAudioDevice();
		if (Handle->ActiveSound.bLocationDefined)
		{
			float MaxDistance = 0.0f;
			float FocusFactor = 1.0f;
			const FVector Location = Handle->ActiveSound.Transform.GetLocation();

			const TObjectPtr<USoundBase> Sound = Handle->ActiveSound.GetSound();
			const FSoundAttenuationSettings* AttenuationSettings = Sound->GetAttenuationSettingsToApply();
			AudioDevice->GetMaxDistanceAndFocusFactor(Sound, GetWorld(), Location, AttenuationSettings, MaxDistance, FocusFactor);
			
			Handle->ActiveSound.bHasAttenuationSettings = (AttenuationSettings != nullptr);
			if (Handle->ActiveSound.bHasAttenuationSettings)
			{
				Handle->ActiveSound.AttenuationSettings = *AttenuationSettings;
				Handle->ActiveSound.FocusData.PriorityScale = AttenuationSettings->GetFocusPriorityScale(GetAudioDeviceHandle().GetAudioDevice()->GetGlobalFocusSettings(), FocusFactor);
			}

			Handle->ActiveSound.MaxDistance = MaxDistance;
		}

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const FString* EventLogID = &Audio::Trace::EventLog::ID::PlayRequestSoundHandle;
#else
		const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
		
		AudioDevice->AddNewActiveSound(Handle->ActiveSound, nullptr, EventLogID);
		return Audio::EResult::Success;
	}
	
	return Audio::EResult::Failure;
}

void USoundHandleSubsystem::Stop(const Audio::FSoundHandleID ID)
{	
	if(FSoundHandle* Handle = ActiveHandles.Find(ID))
	{
		FAudioDevice* AudioDevice = GetAudioDeviceHandle().GetAudioDevice();

#if UE_AUDIO_PROFILERTRACE_ENABLED
		const FString* EventLogID = &Audio::Trace::EventLog::ID::StopRequestedSoundHandle;
#else
		const FString* EventLogID = nullptr;
#endif // UE_AUDIO_PROFILERTRACE_ENABLED

		AudioDevice->StopActiveSound(&Handle->ActiveSound, EventLogID);
	}
}
