// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "GameplayTagContainer.h"
#include "AudioAssetUserData.generated.h"

#define UE_API AUDIOGAMEPLAY_API

/**
 * UAudioAssetUserData - Base class for user data being attached to audio assets
 */
UCLASS(MinimalAPI, Blueprintable)
class UAudioAssetUserData : public UAssetUserData
{
	GENERATED_BODY()
	
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Tags")
	FGameplayTagContainer MetadataTags;

	// Gather all AudioAssetUserData tags for this sound and its sound class. 
	UFUNCTION(BlueprintPure, Category = "Audio AssetUserData Tags")
	static UE_API FGameplayTagContainer GetAllTags(USoundBase* InSound);

	// Whether the sound or its sound class have the supplied (possibly exact) tag .
	UFUNCTION(BlueprintPure, Category = "Audio AssetUserData Tags")
	static UE_API bool HasTag(USoundBase* InSound, FGameplayTag InTag, bool bExactMatch = false);

	// Gather all AudioAssetUserData tags for this sound and its sound class, that match the supplied tag.
	UFUNCTION(BlueprintPure, Category = "Audio AssetUserData Tags")
	static UE_API FGameplayTagContainer GetFilteredTags(USoundBase* InSound, FGameplayTag InTag);

	// If the supplied object has a UAudioAssetUserData asset user data attached, gets it. Returns nullptr otherwise.
	static UE_API UAudioAssetUserData* Get(UObject* InObject);
};

#undef UE_API
