// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioComponentGroupExtension.h"
#include "AudioParameter.h"
#include "AudioParameterControllerInterface.h"
#include "Components/SceneComponent.h"
#include "AudioComponentGroup.generated.h"

#define UE_API AUDIOGAMEPLAY_API

template <typename InterfaceType> class TScriptInterface;

class USoundBase;
class UParamCollection;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FSoundGroupChanged);
DECLARE_DYNAMIC_DELEGATE_OneParam(FSoundCallback, const FName&, EventName);
DECLARE_DYNAMIC_DELEGATE_OneParam(FBoolParamCallback, const bool, ParamValue);
DECLARE_DYNAMIC_DELEGATE_OneParam(FStringParamCallback, const FString&, Value);

/*
 * Automatic Handler for voices and parameters across any number of AudioComponents
 */
UCLASS(MinimalAPI, BlueprintType, DefaultToInstanced)
class UAudioComponentGroup : public USceneComponent, public IAudioParameterControllerInterface
{
	GENERATED_BODY()

public:

	UE_API UAudioComponentGroup(const FObjectInitializer& ObjectInitializer);

	virtual ~UAudioComponentGroup() = default;

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	static UE_API UAudioComponentGroup* StaticGetOrCreateComponentGroup(AActor* Actor);

	UE_API virtual void BeginPlay() override;

	// Stop all instances of this Sound on any internal or external components
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void StopSound(USoundBase* Sound, const float FadeTime = 0.f);

	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API bool IsPlayingAny() const;

	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category = AudioComponentGroup)
	bool IsVirtualized() const { return bIsVirtualized; }

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void BroadcastStopAll();

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void BroadcastKill();

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void BroadcastEvent(const FName EventName);

	// Component interaction
	UE_API UAudioComponent* GetNextAvailableComponent();

	UE_API UAudioComponent* AddComponent();
	UE_API UAudioComponent* ResetComponent(UAudioComponent* Component) const;
	UE_API void RemoveComponent(const UAudioComponent* InComponent);

	//Allow an externally created AudioComponent to share parameters with this SoundGroup
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void AddExternalComponent(UAudioComponent* ComponentToAdd);

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void RemoveExternalComponent(UAudioComponent* ComponentToRemove);

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void EnableVirtualization();

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void DisableVirtualization();

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	void SetVolumeMultiplier(const float InVolume) { GroupModifier.Volume = InVolume; }
	
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	void SetPitchMultiplier(const float InPitch) { GroupModifier.Pitch = InPitch; }

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	void SetLowPassFilter(const float InFrequency) { GroupModifier.LowPassFrequency = InFrequency; }

	const TArray<FAudioParameter>& GetParams() { return PersistentParams; }

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void AddExtension(TScriptInterface<IAudioComponentGroupExtension> NewExtension);
	
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void RemoveExtension(TScriptInterface<IAudioComponentGroupExtension> NewExtension);

	UE_API void UpdateExtensions(const float DeltaTime);

	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;

	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API float GetFloatParamValue(const FName ParamName) const;

	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API bool GetBoolParamValue(const FName ParamName) const;
	 
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API FString GetStringParamValue(const FName ParamName) const;

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void SubscribeToStringParam(const FName ParamName, FStringParamCallback Delegate);

	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void SubscribeToEvent(const FName EventName, FSoundCallback Delegate);
	
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void SubscribeToBool(const FName ParamName, FBoolParamCallback Delegate);

	// remove any string, event, and bool subscriptions that are bound to this object
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = AudioComponentGroup)
	UE_API void UnsubscribeObject(const UObject* Object);
	
	//~ Begin IAudioParameterControllerInterface interface 
	UE_API virtual void ResetParameters() override;

	UE_API virtual void SetTriggerParameter(FName InName) override;
	UE_API virtual void SetBoolParameter(FName InName, bool InBool) override;
	UE_API virtual void SetBoolArrayParameter(FName InName, const TArray<bool>& InValue) override;
	UE_API virtual void SetIntParameter(FName InName, int32 InInt) override;
	UE_API virtual void SetIntArrayParameter(FName InName, const TArray<int32>& InValue) override;
	UE_API virtual void SetFloatParameter(FName InName, float InFloat) override;
	UE_API virtual void SetFloatArrayParameter(FName InName, const TArray<float>& InValue) override;
	UE_API virtual void SetStringParameter(FName InName, const FString& InValue) override;
	UE_API virtual void SetStringArrayParameter(FName InName, const TArray<FString>& InValue) override;
	UE_API virtual void SetObjectParameter(FName InName, UObject* InValue) override;
	UE_API virtual void SetObjectArrayParameter(FName InName, const TArray<UObject*>& InValue) override;

	UE_API virtual void SetParameter(FAudioParameter&& InValue) override;
	UE_API virtual void SetParameters(TArray<FAudioParameter>&& InValues) override;
	UE_API virtual void SetParameters_Blueprint(const TArray<FAudioParameter>& InParameters) override;
	//~ End IAudioParameterControllerInterface interface

	UPROPERTY(BlueprintAssignable)
	FSoundGroupChanged OnStopped;

	UPROPERTY(BlueprintAssignable)
	FSoundGroupChanged OnKilled;

	UPROPERTY(BlueprintAssignable)
	FSoundGroupChanged OnVirtualized;

	UPROPERTY(BlueprintAssignable)
	FSoundGroupChanged OnUnvirtualized;

	UE_API void IterateComponents(const TFunction<void(UAudioComponent*)> OnIterate);

protected:

	UE_API void ApplyParams(UAudioComponent* Component) const;
	UE_API void ApplyModifiers(UAudioComponent* Component, const FAudioComponentModifier& Modifier) const;

	UE_API void UpdateComponentParameters();

	UE_API float GetComponentVolume() const;

	UE_API void ExecuteStringParamSubscriptions(const FAudioParameter& StringParam);
	UE_API void ExecuteBoolParamSubscriptions(const FAudioParameter& BoolParam);
	UE_API void ExecuteEventSubscriptions(const FName EventName);

	UE_API const FAudioParameter* GetParamInternal(const FName ParamName) const;

	UPROPERTY(VisibleAnywhere, Category = AudioComponentGroup)
	TArray<TObjectPtr<UAudioComponent>> Components;

	UPROPERTY(Transient)
	TArray<FAudioParameter> ParamsToSet;
	
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = AudioComponentGroup)
	TArray<FAudioParameter> PersistentParams;

	UPROPERTY(Transient)
	TArray<TScriptInterface<IAudioComponentGroupExtension>> Extensions;

	// Modifier set externally via BP functions
	FAudioComponentModifier GroupModifier;

	// final values set last update
	FAudioComponentModifier CachedModifier;

	//Components managed externally that won't be used in the pool, but can still share parameters
	TArray<TWeakObjectPtr<UAudioComponent>> ExternalComponents;

	TMap<FName, TArray<FStringParamCallback>> StringSubscriptions;
	TMap<FName, TArray<FSoundCallback>> EventSubscriptions;
	TMap<FName, TArray<FBoolParamCallback>> BoolSubscriptions;

	bool bIsVirtualized = false;

	friend class FAudioComponentGroupDebug;
};

#undef UE_API
