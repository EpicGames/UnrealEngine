// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

class FString;
struct FAudioParameter;
struct FColor;

class AActor;
class UAudioComponentGroup;

class FAudioComponentGroupDebug
{
public:
	// Print debug info above the component group's owner, depending on which console variables are enabled
	static void AUDIOGAMEPLAY_API DebugPrint(const UAudioComponentGroup* ComponentGroup);

	// print the names and values of parameters set on this component group
	static FString DebugPrintParams(const UAudioComponentGroup* ComponentGroup);

	// return a string with the name and value of an audio parameter, depending on which underlying type is held by the parameter
	static FString GetDebugParamString(const FAudioParameter& Param);

	// green, or yellow if the component group is virtualized
	static FColor GetDebugStringColor(const bool bVirtualized);

	// print a list of all playing sounds on the component group
	static FString DebugPrintActiveSounds(const UAudioComponentGroup* ComponentGroup);

	// print the final debug string above the target actor
	static void PrintDebugString(const FString& InString, AActor* Owner, const FColor DrawColor);
};
