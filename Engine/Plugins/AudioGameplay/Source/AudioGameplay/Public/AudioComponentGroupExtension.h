// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "UObject/Interface.h"
#include "AudioComponentGroupExtension.generated.h"

class UAudioComponent;
class UAudioComponentGroup;

struct FAudioComponentModifier
{
	float Volume = 1.f;
	float Pitch = 1.f;
	float LowPassFrequency = MAX_FILTER_FREQUENCY;

	void Combine(const FAudioComponentModifier& Other)
	{
		Volume *= Other.Volume;
		Pitch *= Other.Pitch;
		LowPassFrequency = FMath::Min(LowPassFrequency, Other.LowPassFrequency);
	}

	bool IsNearlyEqual(const FAudioComponentModifier& Other) const
	{
		return FMath::IsNearlyEqual(Volume, Other.Volume)
			&& FMath::IsNearlyEqual(Pitch, Other.Pitch)
			&& FMath::IsNearlyEqual(LowPassFrequency, Other.LowPassFrequency);
	}

	static FAudioComponentModifier& Default()
	{
		static FAudioComponentModifier DefaultModifier;
		return DefaultModifier;
	}
};

UINTERFACE(MinimalAPI, BlueprintType)
class UAudioComponentGroupExtension : public UInterface
{
	GENERATED_BODY()
};

class IAudioComponentGroupExtension : public IInterface
{
	GENERATED_BODY()

public:

	virtual void Update(const float DeltaTime, UAudioComponentGroup* Group, FAudioComponentModifier& OutModifier) {}

	virtual void OnAddedToGroup(UAudioComponentGroup* NewGroup) {}

	virtual void OnComponentAdded(UAudioComponent* NewComponent) {}
};
