// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "AudioGameplayFlags.h"
#include "AudioGameplayComponent.generated.h"

#define UE_API AUDIOGAMEPLAY_API

namespace EEndPlayReason { enum Type : int; }

UCLASS(MinimalAPI, Blueprintable, Config = Game, meta = (BlueprintSpawnableComponent))
class UAudioGameplayComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:

	using PayloadFlags = AudioGameplay::EComponentPayload;

	virtual ~UAudioGameplayComponent() = default;

	//~ Begin UActorComponent interface
	UE_API virtual void Activate(bool bReset = false) override;
	UE_API virtual void Deactivate() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End UActorComponent interface

	UE_API virtual bool HasPayloadType(PayloadFlags InType) const;

protected:

	UE_API virtual void Enable();
	UE_API virtual void Disable();

	PayloadFlags PayloadType = PayloadFlags::AGCP_None;
};

#undef UE_API
