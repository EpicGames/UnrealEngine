// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

namespace AudioGameplay
{
	enum class EComponentPayload : uint8
	{
		/** No payload */
		AGCP_None = 0x00,

		/** Payload affects audio listeners */
		AGCP_Listener = 0x01,

		/** Payload affects Active Sound playback */
		AGCP_ActiveSound = 0x02,
	};
};

ENUM_CLASS_FLAGS(AudioGameplay::EComponentPayload);