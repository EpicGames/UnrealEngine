// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "GameplayTagContainer.h"

#include "AudioGameplayRequirements.generated.h"

#define UE_API AUDIOGAMEPLAY_API

/**
 * UAudioRequirementPreset - A reusable GameplayTagQuery for audio
 */
UCLASS(MinimalAPI, BlueprintType)
class UAudioRequirementPreset : public UDataAsset
{
	GENERATED_UCLASS_BODY()

public:

	~UAudioRequirementPreset() = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gameplay Tag Query")
	FGameplayTagQuery Query;

	/** Returns true if the query is not empty and matches the provided tags */
	UE_API bool Matches(const FGameplayTagContainer& Tags) const;
};

/**
 * AudioGameplayRequirements - A set of requirements for audio gameplay features.
 */
USTRUCT(BlueprintType)
struct FAudioGameplayRequirements
{
	GENERATED_USTRUCT_BODY()

	/** An optional requirement preset tested against a collection of tags to determine a match.  Ignored if null. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Requirements")
	TObjectPtr<UAudioRequirementPreset> Preset = nullptr;

	/** An optional custom query tested against a collection of tags to determine a match.  Ignored if empty. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Requirements")
	FGameplayTagQuery Custom;

	/** Returns true if the preset and the custom query match the provided tags; ignores preset or custom query if they are empty. */
	UE_API bool Matches(const FGameplayTagContainer& Tags) const;
};

#undef UE_API
