// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#define UE_API AUDIOGAMEPLAY_API

namespace UE::Audio
{
	/**
	 * FShuffleUtil - Utility to provide a fast random index in an array without replacement
	 *   ie, all elements will be returned from GetNextIndex exactly once before repeating
	 */
	struct FShuffleUtil
	{
		/** Initializes the Shuffle Array */
		UE_API void Initialize(int32 ArraySize);

		/** Returns next valid index, or INDEX_NONE if not initialized. Shuffles array if needed */
		UE_API uint8 GetNextIndex();

	protected:

		uint8 ShuffleMarker = 0;
		TArray<uint8> ShuffleArray;

		/** Shuffles the internal array */
		UE_API void Shuffle();
	};
} // namespace UE::Audio

#undef UE_API
