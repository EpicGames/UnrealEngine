// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "MapAnyKey.h"
#include "Subsystems/WorldSubsystem.h"

#include "AudioGameplayTagCacheSubsystem.generated.h"

#define UE_API AUDIOGAMEPLAY_API

using FGameplayTagMap = UE::AudioGameplay::TMapAnyKey<FGameplayTag>;

/**
 * UAudioGameplayTagCacheSubsystem - Per world subsystem used to persist gameplay tags that are expensive to construct dynamically from parts
 */
UCLASS(MinimalAPI)
class UAudioGameplayTagCacheSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public: 

	UAudioGameplayTagCacheSubsystem() = default;
	virtual ~UAudioGameplayTagCacheSubsystem() = default;

	//~ Begin USubsystem interface
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Deinitialize() override;
	//~ End USubsystem interface

	static UE_API UAudioGameplayTagCacheSubsystem* Get(const UWorld* WorldContext);

	FGameplayTagMap& GetTagCache()
	{
		return GameplayTagCache;
	}

protected:

	FGameplayTagMap GameplayTagCache;
};

#undef UE_API
