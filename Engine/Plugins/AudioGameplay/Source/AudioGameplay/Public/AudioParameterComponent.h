// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioGameplayComponent.h"
#include "AudioParameter.h"
#include "AudioParameterControllerInterface.h"
#include "Audio/ActorSoundParameterInterface.h"

#include "AudioParameterComponent.generated.h"

#define UE_API AUDIOGAMEPLAY_API


DECLARE_LOG_CATEGORY_EXTERN(LogAudioParameterComponent, Log, All);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnParameterChanged, const FAudioParameter&, Parameter);

// Forward Declarations
class UAudioComponent;

/**
 *	UAudioParameterComponent - Can be used to set/store audio parameters and automatically dispatch them (through ActorSoundParameterInterface) 
 *  to any sounds played by the component's Owner Actor
 */
UCLASS(MinimalAPI, BlueprintType, HideCategories=(Object, ActorComponent, Physics, Rendering, Mobility, LOD), meta=(BlueprintSpawnableComponent))
class UAudioParameterComponent : public UAudioGameplayComponent, 
		  										   public IAudioParameterControllerInterface, 
												   public IActorSoundParameterInterface
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = Parameters)
	const TArray<FAudioParameter>& GetParameters() const { return Parameters; }

	/** Begin IActorSoundParameterInterface */
	UE_API virtual void GetActorSoundParams_Implementation(TArray<FAudioParameter>& Params) const override;
	/** End IActorSoundParameterInterface */

	/** Begin IAudioParameterControllerInterface */
	UE_API virtual void ResetParameters() override;
	UE_API virtual void SetTriggerParameter(FName InName) override;
	UE_API virtual void SetBoolParameter(FName InName, bool InValue) override;
	UE_API virtual void SetBoolArrayParameter(FName InName, const TArray<bool>& InValue) override;
	UE_API virtual void SetIntParameter(FName InName, int32 InInt) override;
	UE_API virtual void SetIntArrayParameter(FName InName, const TArray<int32>& InValue) override;
	UE_API virtual void SetFloatParameter(FName InName, float InValue) override;
	UE_API virtual void SetFloatArrayParameter(FName InName, const TArray<float>& InValue) override;
	UE_API virtual void SetStringParameter(FName InName, const FString& InValue) override;
	UE_API virtual void SetStringArrayParameter(FName InName, const TArray<FString>& InValue) override;
	UE_API virtual void SetObjectParameter(FName InName, UObject* InValue) override;
	UE_API virtual void SetObjectArrayParameter(FName InName, const TArray<UObject*>& InValue) override;
	UE_API virtual void SetParameters_Blueprint(const TArray<FAudioParameter>& InParameters) override;
	UE_API virtual void SetParameter(FAudioParameter&& InValue) override;
	UE_API virtual void SetParameters(TArray<FAudioParameter>&& InValues) override;
	/** End IAudioParameterControllerInterface */

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintAssignable)
	FOnParameterChanged OnParameterChanged;
#endif // WITH_EDITORONLY_DATA

private:

	UE_API void SetParameterInternal(FAudioParameter&& InParam);
	UE_API void GetAllAudioComponents(TArray<UAudioComponent*>& Components) const;
	UE_API void LogParameter(FAudioParameter& InParam);

	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<UAudioComponent>> ActiveComponents;

	UPROPERTY(EditDefaultsOnly, Category = "Parameters")
	TArray<FAudioParameter> Parameters;
};

#undef UE_API
