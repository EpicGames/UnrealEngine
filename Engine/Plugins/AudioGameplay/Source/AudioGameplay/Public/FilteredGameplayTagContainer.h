// Copyright Epic Games, Inc. All Rights Reserved.

#include "Delegates/Delegate.h" 
#include "GameplayTagContainer.h"

#define UE_API AUDIOGAMEPLAY_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnFilteredGameplayTagAdded, const FGameplayTag)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnFilteredGameplayTagRemoved, const FGameplayTag)

namespace FilteredGPTContainer
{
	typedef TPair<FGameplayTag, FGameplayTagQuery> FGPTagQueryPair;
}

/*
*	A container of FGameplayTag associated to a FGameplayTagQuery. 
*	Tags will only be added if the accompanying query is satisfied. 
*	Whenever a tag is added or removed, the tags in the container are "Filtered":
*	i.e. their query is checked to see if they should still remain in the array. 
* 
*	Note:
*	Tags are unique
*	Filtering is evaluated from last to first element in the array
*/
class FFilteredGameplayTagContainer
{
public:

	/**
	* Adds a tag to the container if the query matches. 
	* If the tag is added, proceeds to evaluate the conditions on the contained tags to check 
	* if they should remain or not.
	*
	* @param InPair			The tag and query to try and add to this container
	*
	* @return True if tag was added
	*/
	UE_API bool AddTagFiltered(const FilteredGPTContainer::FGPTagQueryPair& InPair);
	
	/**
	* Adds a tag to the container if the query matches.
	* If the tag is added, proceeds to evaluate the conditions on the contained tags to check
	* if they should be kept or not.
	*
	* @param InTag			The tag to try add to this container
	* @param InQuery		The query to determine if this tag should be added to the container
	*
	* @return True if tag was added
	*/
	UE_API bool AddTagFiltered(const FGameplayTag& InTag, const FGameplayTagQuery& InQuery = FGameplayTagQuery());

	/**
	* Removes a tag from the container, then proceeds to evaluate the conditions on the contained tags to check
	* if they should be kept or not.
	*
	* @return True if tag was removed
	*/
	UE_API bool RemoveTagFiltered(const FGameplayTag& InTag);

	inline int32 Find(const FilteredGPTContainer::FGPTagQueryPair& EventToFind) const
	{
		return GPTagQueryPairContainer.Find(EventToFind);
	}

	inline int32 Num() const
	{
		return GPTagQueryPairContainer.Num();
	}

	inline bool IsEmpty() const
	{
		return GPTagQueryPairContainer.Num() == 0;
	}

	FOnFilteredGameplayTagAdded OnGameplayTagAdded;
	FOnFilteredGameplayTagRemoved OnGameplayTagRemoved;

private:

	UE_API bool AddTagToContainers(const FilteredGPTContainer::FGPTagQueryPair& InPair);
	UE_API bool RemoveTagFromContainers(const FGameplayTag& Tag);
	UE_API void FilterTags();
	UE_API void CheckContainersConsistency();

	TArray<FilteredGPTContainer::FGPTagQueryPair> GPTagQueryPairContainer;
	FGameplayTagContainer TagContainer;
};

#undef UE_API
