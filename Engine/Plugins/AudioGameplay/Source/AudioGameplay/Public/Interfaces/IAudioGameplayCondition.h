// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAudioGameplayCondition.generated.h"

/**
 * Interface for implementing conditions for the audio toggle system.
 *
 * Make a component or actor implement this interface to provide its Audio Toggle component
 * set in Arbitrary condition mode a way to compute its On/Off state.
 * If the actor implements it, the Audio Toggle will abide by the implementation on the actor.
 * If any component on the same actor implements it, the Audio Toggle will abide by the
 * implementation on the *first* of these components.
 *
 * See: Audio Toggle component (UAudioGameplayVolumeComponent)
 */
UINTERFACE(BlueprintType, MinimalAPI)
class UAudioGameplayCondition : public UInterface
{
	GENERATED_BODY()
};

class IAudioGameplayCondition
{
	GENERATED_BODY()

public:

	/**
	 * Generic condition check, whatever the toggle is being evaluated for (Audio Listener,
	 * Active sound,...)
	 * 
	 * @returns true if toggle state should be On, false otherwise
	 */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "AudioGameplayCondition")
	AUDIOGAMEPLAY_API bool ConditionMet() const;
	virtual bool ConditionMet_Implementation() const { return false; }
	
	/**
	 * Allows testing a condition against a provided position.
	 * This position can be that of the Audio Listener, of the Active Sound,... for which the
	 * toggle is being evaluated.
	 *
	 * @param Position - The location to be considered
	 * 
	 * @returns true if toggle state should be On for this entity's position, false otherwise
	 */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "AudioGameplayCondition")
	AUDIOGAMEPLAY_API bool ConditionMet_Position(const FVector& Position) const;
	virtual bool ConditionMet_Position_Implementation(const FVector& Position) const { return false; }
};
