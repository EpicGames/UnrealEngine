// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAudioGameplayVolumeInteraction.generated.h"

/**
 * Interface for interacting with the audio toggle system.
 *
 * Make a component implement this interface to receive notifications when the toggle is
 * active for at least one listener or inactive for all.
 *
 * The owning actor must have an Audio Toggle component for other components of this actor
 * implementing this interface to receive the notifications.
 *
 * Instead of implementing this interface, you can also just bind to the "On Toggled On" and
 * "On Toggled Off" events of the Audio Toggle component directly.
 */
UINTERFACE(BlueprintType, MinimalAPI, meta = (DisplayName = "Audio Toggle Interaction"))
class UAudioGameplayVolumeInteraction : public UInterface
{
	GENERATED_BODY()
};

class IAudioGameplayVolumeInteraction
{
	GENERATED_BODY()

public:

	/**
	 * Called when the Audio Toggle on the same actor switches to ON.
	 */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "AudioGameplayVolume", meta = (DisplayName = "On Toggled On"))
	AUDIOGAMEPLAY_API void OnListenerEnter();
	virtual void OnListenerEnter_Implementation() {}

	/**
	 * Called when the Audio Toggle on the same actor switches to OFF.
	 */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "AudioGameplayVolume", meta = (DisplayName = "On Toggled Off"))
	AUDIOGAMEPLAY_API void OnListenerExit();
	virtual void OnListenerExit_Implementation() {}
};
