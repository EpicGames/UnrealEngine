// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Map.h"

namespace UE::AudioGameplay
{
	/**
	 * Variant of TMap that allows multiple key types within the same map instance.
	 * Key types must be hash-able, as with a regular map.
	 */
	template <typename ValueType>
	class TMapAnyKey : public TMap<uint32, ValueType>
	{
		using Super = TMap<uint32, ValueType>;

	public:

		/**
		 * Find the value associated with a specified key, or if none exists, 
		 * add a value using the default constructor.
		 *
		 * @param Key The key to search for.
		 * @param bHadKey True if found, false if added.
		 * @return A reference to the value associated with the specified key.
		 */
		template <typename AnyKeyType>
		ValueType& FindOrAdd(AnyKeyType Key, bool& bHadKey)
		{
			// Calculate the hash for ANY key, using the same method a TMap<> would use for that key.
			uint32 KeyHash = TSet<AnyKeyType>::KeyFuncsType::GetKeyHash(Key);

			// Use the hashes directly as keys for the underlying TMap parent class.
			if (ValueType* Ptr = Super::Find(KeyHash))
			{
				bHadKey = true;
				return *Ptr;
			}

			bHadKey = false;
			return Super::Add(KeyHash);
		}
	};
}
