// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActiveSound.h"
#include "ActiveSoundUpdateInterface.h"
#include "Audio/ISoundHandleSystem.h"
#include "Subsystems/AudioEngineSubsystem.h"

#include "SoundHandleSubsystem.generated.h"

class ISoundHandleOwner;

/**
 * An implementation of ISoundHandleSystem using AudioEngineSubsystem
 */
UCLASS()
class USoundHandleSubsystem : public UAudioEngineSubsystem
	, public IActiveSoundUpdateInterface
	, public ISoundHandleSystem
{
	GENERATED_BODY()

public:
	virtual ~USoundHandleSubsystem() override = default;

	//~ Begin USubsystem interface
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem interface
	
	//~ Begin IActiveSoundUpdateInterface
	virtual void NotifyActiveSoundDeleting(const FActiveSound& ActiveSound) override;
	virtual void NotifyVirtualizedSoundDeleting(const FActiveSound& ActiveSound) override;
	//~ End IActiveSoundUpdateInterface

	//~ Begin ISoundHandleSystem
	virtual Audio::FSoundHandleID CreateSoundHandle(USoundBase* Sound, ISoundHandleOwner* Owner) override;
	virtual void SetTransform(Audio::FSoundHandleID ID, const FTransform& Transform) override;
	virtual Audio::EResult Play(Audio::FSoundHandleID ID) override;
	virtual void Stop(Audio::FSoundHandleID ID) override;
	//~ End ISoundHandleSystem

protected:
	struct FSoundHandle
	{
		FActiveSound ActiveSound;
	};
	
	TMap<Audio::FSoundHandleID, FSoundHandle> ActiveHandles;
	TMap<Audio::FSoundHandleID, ISoundHandleOwner*> Owners;

	void NotifySoundDeleting(const FActiveSound& ActiveSound);
};
