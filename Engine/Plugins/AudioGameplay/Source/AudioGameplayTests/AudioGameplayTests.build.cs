// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioGameplayTests : ModuleRules
{
	public AudioGameplayTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AudioExtensions",
				"AudioGameplay",
				"Core",
				"GameplayTags",
			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
			}
			);


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}