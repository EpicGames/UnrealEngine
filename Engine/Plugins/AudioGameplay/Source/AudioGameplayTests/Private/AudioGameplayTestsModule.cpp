// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayTestsModule.h"
#include "Modules/ModuleManager.h"

void FAudioGameplayTestsModule::StartupModule()
{
}

void FAudioGameplayTestsModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FAudioGameplayTestsModule, AudioGameplayTests)