// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "FilteredGameplayTagContainer.h"


namespace AudioGameplayTagsTestParams
{
	static const FName TestTagName1 = "AudioGameplay.Tests.FilteredGameplayTagContainer.TestTag1";
	static const FName TestTagName2 = "AudioGameplay.Tests.FilteredGameplayTagContainer.TestTag2";
	static const FName TestTagName3 = "AudioGameplay.Tests.FilteredGameplayTagContainer.TestTag3";
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioGameplayTagTests_TestAddTagFiltered, "AudioGameplay.FilteredGameplayTags.TestAddTagFiltered", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioGameplayTagTests_TestAddTagFiltered::RunTest(const FString& Parameters)
{
	FGameplayTag FilteredTag1 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName1);
	FGameplayTag FilteredTag2 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName3);

	FFilteredGameplayTagContainer TestContainer;

	bool AddTagFilteredReturn = true;

	AddTagFilteredReturn &= TestContainer.AddTagFiltered(FilteredTag1);
	bool bExpectedNumToBeOne = TestContainer.Num() == 1;

	AddTagFilteredReturn &= TestContainer.AddTagFiltered(FilteredTag2);
	bool bExpectedNumToBeTwo = TestContainer.Num() == 2;

	return AddTagFilteredReturn && bExpectedNumToBeOne && bExpectedNumToBeTwo;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioGameplayTagTests_TestRemoveTagFiltered, "AudioGameplay.FilteredGameplayTags.TestRemoveTagFiltered", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioGameplayTagTests_TestRemoveTagFiltered::RunTest(const FString& Parameters)
{
	FGameplayTag FilteredTag1 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName1);
	FGameplayTag FilteredTag2 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName3);

	FFilteredGameplayTagContainer TestContainer;

	TestContainer.AddTagFiltered(FilteredTag1);
	TestContainer.AddTagFiltered(FilteredTag2);

	bool bExpectedNumToBeTwo = TestContainer.Num() == 2;

	bool bRemoveTagFilteredReturn = TestContainer.RemoveTagFiltered(FilteredTag1);
	bool bExpectedNumToBeOne = TestContainer.Num() == 1;

	return bRemoveTagFilteredReturn && bExpectedNumToBeOne && bExpectedNumToBeTwo;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioGameplayTagTests_TestAddWithSameName, "AudioGameplay.FilteredGameplayTags.TestAddWithSameName", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioGameplayTagTests_TestAddWithSameName::RunTest(const FString& Parameters)
{
	FGameplayTag FilteredTag1 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName1);

	FFilteredGameplayTagContainer TestContainer;

	const bool bFirstAddPass = TestContainer.AddTagFiltered(FilteredTag1);
	const bool bSecondAddFails = !TestContainer.AddTagFiltered(FilteredTag1);
	bool bExpectedNumToBeOne = TestContainer.Num() == 1;

	return bFirstAddPass && bSecondAddFails && bExpectedNumToBeOne;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioGameplayTagTests_TestRemoveWithNoMatch, "AudioGameplay.FilteredGameplayTags.TestRemoveWithNoMatch", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioGameplayTagTests_TestRemoveWithNoMatch::RunTest(const FString& Parameters)
{
	FGameplayTag FilteredTag1 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName1);

	FFilteredGameplayTagContainer TestContainer;

	const bool bRemoveFails = !TestContainer.RemoveTagFiltered(FilteredTag1);
	const bool bEmptyContainer = TestContainer.IsEmpty();


	return bRemoveFails && bEmptyContainer;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioGameplayTagTests_TestAddTagFilteredWithInvalidCondition, "AudioGameplay.FilteredGameplayTags.TestAddTagFilteredWithInvalidCondition", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioGameplayTagTests_TestAddTagFilteredWithInvalidCondition::RunTest(const FString& Parameters)
{
	FGameplayTag FilteredTag1 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName1);

	FGameplayTagQuery NotFilteredTag1Query = FGameplayTagQuery::BuildQuery(FGameplayTagQueryExpression().NoTagsMatch().AddTag(FilteredTag1));

	FilteredGPTContainer::FGPTagQueryPair FilteredTag2(FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName2), NotFilteredTag1Query);

	FFilteredGameplayTagContainer TestContainer;
	TestContainer.AddTagFiltered(FilteredTag1);
	bool bAddTagFilteredReturn = TestContainer.AddTagFiltered(FilteredTag2);

	return !bAddTagFilteredReturn;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioGameplayTagTests_TestEventRemovedWhenConditionBecomesFalseFromAdding, "AudioGameplay.FilteredGameplayTags.TestEventRemovedWhenConditionBecomesFalseFromAdding", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioGameplayTagTests_TestEventRemovedWhenConditionBecomesFalseFromAdding::RunTest(const FString& Parameters)
{
	FGameplayTag FilteredTag1 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName1);

	FGameplayTagQuery NotFilteredTag1Query = FGameplayTagQuery::BuildQuery(FGameplayTagQueryExpression().NoTagsMatch().AddTag(FilteredTag1));	

	FilteredGPTContainer::FGPTagQueryPair FilteredTag2(FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName2), NotFilteredTag1Query);

	FFilteredGameplayTagContainer TestContainer;
	TestContainer.AddTagFiltered(FilteredTag2);
	bool bFilteredTag2InitiallyAdded = TestContainer.Find(FilteredTag2) != INDEX_NONE; 
	TestContainer.AddTagFiltered(FilteredTag1);
	bool bFilteredTag2NoLongerPresent = TestContainer.Find(FilteredTag2) == INDEX_NONE;

	return bFilteredTag2InitiallyAdded && bFilteredTag2NoLongerPresent;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAudioGameplayTagTests_TestEventRemovedWhenConditionBecomesFalseFromRemoving, "AudioGameplay.FilteredGameplayTags.TestEventRemovedWhenConditionBecomesFalseFromRemoving", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAudioGameplayTagTests_TestEventRemovedWhenConditionBecomesFalseFromRemoving::RunTest(const FString& Parameters)
{
	FGameplayTag FilteredTag1 = FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName1);

	FGameplayTagQuery FilteredTag1PresentQuery = FGameplayTagQuery::BuildQuery(FGameplayTagQueryExpression().AnyTagsMatch().AddTag(FilteredTag1));

	FilteredGPTContainer::FGPTagQueryPair FilteredTag2(FGameplayTag::RequestGameplayTag(AudioGameplayTagsTestParams::TestTagName2), FilteredTag1PresentQuery);

	FFilteredGameplayTagContainer TestContainer;
	TestContainer.AddTagFiltered(FilteredTag1);
	TestContainer.AddTagFiltered(FilteredTag2);
	bool bFilteredTag2InitiallyAdded = TestContainer.Find(FilteredTag2) != INDEX_NONE;
	bool bExpectedNumToBeTwo = TestContainer.Num() == 2;

	TestContainer.RemoveTagFiltered(FilteredTag1);
	bool bFilteredTag2NoLongerPresent = TestContainer.Find(FilteredTag2) == INDEX_NONE;
	bool bExpectedNumToBeZero = TestContainer.Num() == 0;

	return bFilteredTag2InitiallyAdded && bFilteredTag2NoLongerPresent && bExpectedNumToBeTwo && bExpectedNumToBeZero;
}

#endif